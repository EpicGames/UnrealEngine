// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class OpenColorIOWrapper : ModuleRules
	{
		public OpenColorIOWrapper(ReadOnlyTargetRules Target) : base(Target)
		{
			PCHUsage = PCHUsageMode.UseExplicitOrSharedPCHs;

			PrivateIncludePathModuleNames.AddRange(new string[]
			{
				"Engine", // for TextureDefines.h
			});

			PublicDependencyModuleNames.AddRange(new string[]
			{
				"Core",
			});

			bool bIsSupported = false;

			// Because the servers run with NullRHI, we currently ignore OCIO color operations on this build type.
			if (Target.Type != TargetType.Server)
			{
				// Mirror OpenColorIOLib platform coverage
				if (Target.Platform == UnrealTargetPlatform.Win64 ||
					Target.IsInPlatformGroup(UnrealPlatformGroup.Unix) ||
					Target.Platform == UnrealTargetPlatform.Mac)
				{
					PrivateDependencyModuleNames.AddRange(new string[]
					{
						"ImageCore",
					});

					PrivateDefinitions.Add("OpenColorIO_SKIP_IMPORTS");
					AddEngineThirdPartyPrivateStaticDependencies(Target, "OpenColorIOLib");

					bIsSupported = true;
				}
			}
			
			PublicDefinitions.Add("WITH_OCIO=" + (bIsSupported ? "1" : "0"));
		}
	}
}
