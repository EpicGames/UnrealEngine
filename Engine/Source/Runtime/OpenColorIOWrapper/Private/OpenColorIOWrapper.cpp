// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpenColorIOWrapper.h"
#include "OpenColorIOWrapperModule.h"

#if WITH_OCIO

#include "Containers/StringView.h"
#include "Containers/UnrealString.h"
#include "Containers/Map.h"
#include "ColorManagement/ColorSpace.h"
#include "ColorManagement/TransferFunctions.h"
#include "Engine/TextureDefines.h"
#include "ImageCore.h"
#include "ImageParallelFor.h"

THIRD_PARTY_INCLUDES_START
#include "OpenColorIO/OpenColorIO.h"
THIRD_PARTY_INCLUDES_END

#define LOCTEXT_NAMESPACE "OpenColorIOWrapper"

#if PLATFORM_EXCEPTIONS_DISABLED
	#define OCIO_EXCEPTION_HANDLING_TRY()
	#define OCIO_EXCEPTION_HANDLING_CATCH(Verbosity, Format, ...) (0)
	#define OCIO_EXCEPTION_HANDLING_CATCH_ERROR()
#else
	#define OCIO_EXCEPTION_HANDLING_TRY() \
		try {
	
	// NOTE: A string parameter must always be added to the format string since we automatically include the error message.
	#define OCIO_EXCEPTION_HANDLING_CATCH(Verbosity, Format, ...) \
		} catch (OCIO_NAMESPACE::Exception& Exc) { \
			UE_LOG(LogOpenColorIOWrapper, Verbosity, Format, ##__VA_ARGS__, StringCast<TCHAR>(Exc.what()).Get()); \
		}

	#define OCIO_EXCEPTION_HANDLING_CATCH_ERROR() OCIO_EXCEPTION_HANDLING_CATCH(Error, TEXT("%s"))
#endif

namespace OpenColorIOWrapper
{
	// Build routine since there is no FAnsiString
	TUniquePtr<ANSICHAR[]> MakeAnsiString(const TCHAR* Str)
	{
		int32 Num = FPlatformString::ConvertedLength<ANSICHAR>(Str);
		TUniquePtr<ANSICHAR[]> Ret =  MakeUnique<ANSICHAR[]>(Num);
		FMemory::Memcpy(Ret.Get(), StringCast<ANSICHAR>(Str).Get(), Num);
		return Ret;
	}
}

const TCHAR* OpenColorIOWrapper::GetVersion()
{
	return TEXT(OCIO_VERSION);
}

uint32 OpenColorIOWrapper::GetVersionHex()
{
	return OCIO_VERSION_HEX;
}

void OpenColorIOWrapper::ClearAllCaches()
{
	OCIO_NAMESPACE::ClearAllCaches();
}

struct FOpenColorIOConfigPimpl
{
	OCIO_NAMESPACE::ConstConfigRcPtr Config = nullptr;
};

struct FOpenColorIOProcessorPimpl
{
	OCIO_NAMESPACE::ConstProcessorRcPtr Processor = nullptr;

	/** Get processor optimization flags. */
	static OCIO_NAMESPACE::OptimizationFlags GetOptimizationFlags()
	{
		return static_cast<OCIO_NAMESPACE::OptimizationFlags>(
			OCIO_NAMESPACE::OptimizationFlags::OPTIMIZATION_DEFAULT |
			OCIO_NAMESPACE::OptimizationFlags::OPTIMIZATION_NO_DYNAMIC_PROPERTIES
		);
	}
};

struct FOpenColorIOGPUProcessorPimpl
{
	OCIO_NAMESPACE::ConstGPUProcessorRcPtr Processor = nullptr;
	OCIO_NAMESPACE::GpuShaderDescRcPtr ShaderDescription = nullptr;
};


namespace {
	bool IsImageFormatSupported(const FImageView& InImage)
	{
		switch (InImage.Format)
		{
		case ERawImageFormat::BGRA8:
		case ERawImageFormat::RGBA16:
		case ERawImageFormat::RGBA16F:
		case ERawImageFormat::RGBA32F:
			return true;
		default:
			return false;
		}
	}

	OCIO_NAMESPACE::PackedImageDesc GetImageDesc(const FImageView& InImage)
	{
		OCIO_NAMESPACE::ChannelOrdering Ordering = OCIO_NAMESPACE::ChannelOrdering::CHANNEL_ORDERING_RGBA;
		OCIO_NAMESPACE::BitDepth BitDepth = OCIO_NAMESPACE::BitDepth::BIT_DEPTH_UNKNOWN;

		switch (InImage.Format)
		{
		case ERawImageFormat::BGRA8:
			Ordering = OCIO_NAMESPACE::CHANNEL_ORDERING_BGRA;
			BitDepth = OCIO_NAMESPACE::BIT_DEPTH_UINT8;
			break;
		case ERawImageFormat::RGBA16:
			Ordering = OCIO_NAMESPACE::CHANNEL_ORDERING_RGBA;
			BitDepth = OCIO_NAMESPACE::BIT_DEPTH_UINT16;
			break;
		case ERawImageFormat::RGBA16F:
			Ordering = OCIO_NAMESPACE::CHANNEL_ORDERING_RGBA;
			BitDepth = OCIO_NAMESPACE::BIT_DEPTH_F16;
			break;
		case ERawImageFormat::RGBA32F:
			Ordering = OCIO_NAMESPACE::CHANNEL_ORDERING_RGBA;
			BitDepth = OCIO_NAMESPACE::BIT_DEPTH_F32;
			break;
		default:

			// All other cases should have been eliminated by calling IsImageFormatSupported() previously.
			checkNoEntry();
		}

		return OCIO_NAMESPACE::PackedImageDesc(
			InImage.RawData,
			static_cast<long>(InImage.GetWidth()),
			static_cast<long>(InImage.GetHeight()),
			Ordering,
			BitDepth,
			OCIO_NAMESPACE::AutoStride,
			OCIO_NAMESPACE::AutoStride,
			OCIO_NAMESPACE::AutoStride
			);
	};
}


FOpenColorIOWrapperConfig::FOpenColorIOWrapperConfig()
	: Pimpl(MakePimpl<FOpenColorIOConfigPimpl, EPimplPtrMode::DeepCopy>())
{ }

FOpenColorIOWrapperConfig::FOpenColorIOWrapperConfig(FStringView InFilePath, EAddWorkingColorSpaceOption InOption)
	: FOpenColorIOWrapperConfig()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FOpenColorIOWrapperConfig::FOpenColorIOWrapperConfig);
	OCIO_EXCEPTION_HANDLING_TRY();

		using namespace OCIO_NAMESPACE;

		ConstConfigRcPtr NewConfig = Config::CreateFromFile(StringCast<ANSICHAR>(InFilePath.GetData()).Get());

		if (NewConfig != nullptr)
		{
			if (InOption == WCS_AsInterchangeSpace)
			{
				const TUniquePtr<ANSICHAR[]> AnsiWorkingColorSpaceName = OpenColorIOWrapper::MakeAnsiString(OpenColorIOWrapper::GetWorkingColorSpaceName());
				ConstColorSpaceRcPtr InterchangeCS = NewConfig->getColorSpace(NewConfig->getCanonicalName(OCIO_NAMESPACE::ROLE_INTERCHANGE_SCENE));

				// When the aces interchange color space is present, we add the working color space as an additional option.
				if (InterchangeCS != nullptr && NewConfig->getColorSpace(AnsiWorkingColorSpaceName.Get()) == nullptr)
				{
					ColorSpaceRcPtr WorkingCS = InterchangeCS->createEditableCopy();
					WorkingCS->setName(AnsiWorkingColorSpaceName.Get());
					WorkingCS->setFamily("UE");
					WorkingCS->setDescription("The global engine working color space, as defined in project settings (defaults to sRGB/Rec.709).");
					WorkingCS->clearAliases();

					ConfigRcPtr NewConfigCopy = NewConfig->createEditableCopy();
					NewConfigCopy->addColorSpace(WorkingCS);

					NewConfig = NewConfigCopy;
				}
			}
			else
			{
				UE_LOGF(LogOpenColorIOWrapper, Log, "Could not add the working color space since the config does not define \"aces_interchange\".")
			}
		}

		Pimpl->Config = NewConfig;
	
	OCIO_EXCEPTION_HANDLING_CATCH(Error, TEXT("Could not create OCIO configuration file for %s. Error message: %s"), InFilePath.GetData());
}

bool FOpenColorIOWrapperConfig::IsValid() const
{
	return Pimpl->Config != nullptr;
}

int32 FOpenColorIOWrapperConfig::GetNumColorSpaces() const
{
	if (IsValid())
	{
		return Pimpl->Config->getNumColorSpaces(OCIO_NAMESPACE::SEARCH_REFERENCE_SPACE_ALL, OCIO_NAMESPACE::COLORSPACE_ACTIVE);
	}

	return 0;
}

FString FOpenColorIOWrapperConfig::GetColorSpaceName(int32 Index) const
{
	if (IsValid())
	{
		const char* ColorSpaceName = Pimpl->Config->getColorSpaceNameByIndex(OCIO_NAMESPACE::SEARCH_REFERENCE_SPACE_ALL, OCIO_NAMESPACE::COLORSPACE_ACTIVE, Index);

		return StringCast<TCHAR>(ColorSpaceName).Get();
	}

	return {};
}

int32 FOpenColorIOWrapperConfig::GetColorSpaceIndex(const TCHAR* InColorSpaceName)
{
	if (IsValid())
	{
		return Pimpl->Config->getIndexForColorSpace(StringCast<ANSICHAR>(InColorSpaceName).Get());
	}

	return false;
}

FString FOpenColorIOWrapperConfig::GetColorSpaceFamilyName(const TCHAR* InColorSpaceName) const
{
	if (IsValid())
	{
		OCIO_NAMESPACE::ConstColorSpaceRcPtr ColorSpace = Pimpl->Config->getColorSpace(StringCast<ANSICHAR>(InColorSpaceName).Get());
		
		if (ColorSpace != nullptr)
		{
			return StringCast<TCHAR>(ColorSpace->getFamily()).Get();
		}
	}

	return {};
}

FString FOpenColorIOWrapperConfig::GetColorSpaceDescription(const TCHAR* InColorSpaceName) const
{
	if (IsValid())
	{
		OCIO_NAMESPACE::ConstColorSpaceRcPtr ColorSpace = Pimpl->Config->getColorSpace(StringCast<ANSICHAR>(InColorSpaceName).Get());

		if (ColorSpace != nullptr)
		{
			return StringCast<TCHAR>(ColorSpace->getDescription()).Get();
		}
	}

	return {};
}

int32 FOpenColorIOWrapperConfig::GetNumDisplays() const
{
	if (IsValid())
	{
		return Pimpl->Config->getNumDisplays();
	}

	return 0;
}

FString FOpenColorIOWrapperConfig::GetDisplayName(int32 Index) const
{
	if (IsValid())
	{
		const char* DisplayName = Pimpl->Config->getDisplay(Index);

		return StringCast<TCHAR>(DisplayName).Get();
	}

	return {};
}

int32 FOpenColorIOWrapperConfig::GetNumViews(const TCHAR* InDisplayName) const
{
	if (IsValid())
	{
		return Pimpl->Config->getNumViews(StringCast<ANSICHAR>(InDisplayName).Get());
	}

	return 0;
}

FString FOpenColorIOWrapperConfig::GetViewName(const TCHAR* InDisplayName, int32 Index) const
{
	if (IsValid())
	{
		const char* ViewName = Pimpl->Config->getView(StringCast<ANSICHAR>(InDisplayName).Get(), Index);

		return StringCast<TCHAR>(ViewName).Get();
	}

	return {};
}

FString FOpenColorIOWrapperConfig::GetDisplayViewTransformName(const TCHAR* InDisplayName, const TCHAR* InViewName) const
{
	if (IsValid())
	{
		const char* TransformName = Pimpl->Config->getDisplayViewTransformName(StringCast<ANSICHAR>(InDisplayName).Get(), StringCast<ANSICHAR>(InViewName).Get());

		return StringCast<TCHAR>(TransformName).Get();
	}

	return {};
}

FString FOpenColorIOWrapperConfig::GetDisplayViewDescription(const TCHAR* InDisplayName, const TCHAR* InViewName) const
{
	if (IsValid())
	{
		const char* DisplayViewDescription = Pimpl->Config->getDisplayViewDescription(StringCast<ANSICHAR>(InDisplayName).Get(), StringCast<ANSICHAR>(InViewName).Get());

		return StringCast<TCHAR>(DisplayViewDescription).Get();
	}

	return {};
}

FString FOpenColorIOWrapperConfig::GetDisplayViewTransformDescription(const TCHAR* InDisplayName, const TCHAR* InViewName) const
{
	if (IsValid())
	{
		const char* TransformName = Pimpl->Config->getDisplayViewTransformName(StringCast<ANSICHAR>(InDisplayName).Get(), StringCast<ANSICHAR>(InViewName).Get());

		OCIO_NAMESPACE::ConstViewTransformRcPtr ViewTransform = Pimpl->Config->getViewTransform(TransformName);
		
		if (ViewTransform != nullptr)
		{
			const char* TransformDescription = ViewTransform->getDescription();

			return StringCast<TCHAR>(TransformDescription).Get();
		}
	}

	return {};
}

TMap<FString, FString> FOpenColorIOWrapperConfig::GetCurrentContextStringVars() const
{
	OCIO_EXCEPTION_HANDLING_TRY();
		if (IsValid())
		{
			TMap<FString, FString> ContextStringVars;
			OCIO_NAMESPACE::ConstContextRcPtr CurrentContext = Pimpl->Config->getCurrentContext();

			for (int32 Index = 0; Index < CurrentContext->getNumStringVars(); ++Index)
			{
				const ANSICHAR* Key = CurrentContext->getStringVarNameByIndex(Index);
				const ANSICHAR* Value = CurrentContext->getStringVarByIndex(Index);

				ContextStringVars.Emplace(FString(Key), FString(Value));
			}
			
			return ContextStringVars;
		}
	OCIO_EXCEPTION_HANDLING_CATCH_ERROR();

	return {};
}

FString FOpenColorIOWrapperConfig::GetCacheID() const
{
	OCIO_EXCEPTION_HANDLING_TRY();
		if (IsValid())
		{
			return StringCast<TCHAR>(Pimpl->Config->getCacheID()).Get();
		}
	OCIO_EXCEPTION_HANDLING_CATCH_ERROR();

	return {};
}

FString FOpenColorIOWrapperConfig::GetDebugString() const
{
	TStringBuilder<1024> DebugStringBuilder;
	
	if (IsValid())
	{
		const OCIO_NAMESPACE::ConstConfigRcPtr& Config = Pimpl->Config;
		if (Config->getNumColorSpaces() > 0)
		{
			DebugStringBuilder.Append(TEXT("** ColorSpaces **\n"));
			
			const int32 NumCS = Config->getNumColorSpaces(
				OCIO_NAMESPACE::SEARCH_REFERENCE_SPACE_ALL,   // Iterate over scene & display color spaces.
				OCIO_NAMESPACE::COLORSPACE_ALL);              // Iterate over active & inactive color spaces.

			for (int32 IndexCS = 0; IndexCS < NumCS; ++IndexCS)
			{
				OCIO_NAMESPACE::ConstColorSpaceRcPtr cs = Config->getColorSpace(Config->getColorSpaceNameByIndex(
					OCIO_NAMESPACE::SEARCH_REFERENCE_SPACE_ALL,
					OCIO_NAMESPACE::COLORSPACE_ALL,
					IndexCS));

				DebugStringBuilder.Append(cs->getName());
				DebugStringBuilder.Append(TEXT("\n"));
			}

			DebugStringBuilder.Append(TEXT("** (Display, View) pairs **\n"));

			for (int32 IndexDisplay = 0; IndexDisplay < Config->getNumDisplaysAll(); ++IndexDisplay)
			{
				const ANSICHAR* DisplayName = Config->getDisplayAll(IndexDisplay);

				// Iterate over shared views.
				int32 NumViews = Config->getNumViews(DisplayName);
				for (int IndexView = 0; IndexView < NumViews; ++IndexView)
				{
					const ANSICHAR* ViewName = Config->getView(
						DisplayName,
						IndexView);

					DebugStringBuilder.Append(TEXT("("));
					DebugStringBuilder.Append(DisplayName);
					DebugStringBuilder.Append(TEXT(", "));
					DebugStringBuilder.Append(ViewName);
					DebugStringBuilder.Append(TEXT(")\n"));
				}
			}
		}
	}

	return DebugStringBuilder.ToString();
}

TMap<FString, FString> FOpenColorIOWrapperConfig::GetBuiltInConfigNames(bool bRecommendedOnly, bool bShortenUIName) const
{
	TMap<FString, FString> Names;
	
	const OCIO_NAMESPACE::BuiltinConfigRegistry& Registry = OCIO_NAMESPACE::BuiltinConfigRegistry::Get();
	const int32 NumBuiltIns = Registry.getNumBuiltinConfigs();
	Names.Reserve(NumBuiltIns);

	for (size_t Index = 0; Index < NumBuiltIns; ++Index)
	{		
		if (bRecommendedOnly && !Registry.isBuiltinConfigRecommended(Index))
		{
			continue;
		}

		const ANSICHAR* Name = Registry.getBuiltinConfigName(Index);
		FString UIName = Registry.getBuiltinConfigUIName(Index);
		
		if (bShortenUIName)
		{
			UIName.RemoveFromStart(TEXT("Academy Color Encoding System - "));
		}

		Names.Emplace(Name, MoveTemp(UIName));
	}


	return Names;
}

FString FOpenColorIOWrapperConfig::ResolveConfigPath(const TCHAR* InOriginalPath) noexcept
{
	return OCIO_NAMESPACE::ResolveConfigPath(TCHAR_TO_ANSI(InOriginalPath));
}


FOpenColorIOWrapperEngineConfig::FOpenColorIOWrapperEngineConfig()
	: FOpenColorIOWrapperConfig()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FOpenColorIOWrapperConfig::CreateWorkingColorSpaceToInterchangeConfig)
	using namespace OCIO_NAMESPACE;
	using namespace UE::Color;

	ConfigRcPtr StudioConfig = Config::CreateFromBuiltinConfig("studio-config-v1.0.0_aces-v1.3_ocio-v2.1")->createEditableCopy();

	ColorSpaceRcPtr WCS = ColorSpace::Create();
	WCS->setName(StringCast<ANSICHAR>(OpenColorIOWrapper::GetWorkingColorSpaceName()).Get());
	WCS->setBitDepth(BIT_DEPTH_F32);
	WCS->setEncoding("scene-linear");
	// We know the scene-referred reference space is ACES2065-1, and hence the correct matrix transform.
	const FMatrix44d TransformMat = Transpose<double>(FColorSpaceTransform(FColorSpace::GetWorking(), FColorSpace(EColorSpace::ACESAP0)));
	MatrixTransformRcPtr MatrixTransform = MatrixTransform::Create();
	MatrixTransform->setMatrix(&TransformMat.M[0][0]);
	WCS->setTransform(MatrixTransform, COLORSPACE_DIR_TO_REFERENCE);

	StudioConfig->addColorSpace(WCS);

	Pimpl->Config = StudioConfig;

	check(IsValid());
}

FOpenColorIOWrapperProcessor::FOpenColorIOWrapperProcessor()
	: Pimpl(MakePimpl<FOpenColorIOProcessorPimpl, EPimplPtrMode::DeepCopy>())
{ }

FOpenColorIOWrapperProcessor::FOpenColorIOWrapperProcessor(
	const FOpenColorIOWrapperConfig* InConfig,
	FStringView InSourceColorSpace,
	FStringView InDestinationColorSpace,
	const TMap<FString, FString>& InContextKeyValues)
	: Pimpl(MakePimpl<FOpenColorIOProcessorPimpl, EPimplPtrMode::DeepCopy>())
{
	OCIO_EXCEPTION_HANDLING_TRY();
		if (InConfig != nullptr && InConfig->IsValid())
		{
			const OCIO_NAMESPACE::ConstConfigRcPtr& Config = InConfig->Pimpl->Config;
			OCIO_NAMESPACE::ContextRcPtr Context = Config->getCurrentContext()->createEditableCopy();

			for (const TPair<FString, FString>& KeyValue : InContextKeyValues)
			{
				Context->setStringVar(TCHAR_TO_ANSI(*KeyValue.Key), TCHAR_TO_ANSI(*KeyValue.Value));
			}

			const TCHAR* WCSColorSpaceName = OpenColorIOWrapper::GetWorkingColorSpaceName();
			const TUniquePtr<ANSICHAR[]> AnsiWorkingColorSpaceName = OpenColorIOWrapper::MakeAnsiString(WCSColorSpaceName);

			if (InSourceColorSpace == WCSColorSpaceName)
			{
				Pimpl->Processor = OCIO_NAMESPACE::Config::GetProcessorFromConfigs(
					Context,
					IOpenColorIOWrapperModule::Get().GetEngineBuiltInConfig().Pimpl->Config,
					AnsiWorkingColorSpaceName.Get(),
					Context,
					Config,
					StringCast<ANSICHAR>(InDestinationColorSpace.GetData()).Get());
			}
			else if (InDestinationColorSpace == WCSColorSpaceName)
			{
				Pimpl->Processor = OCIO_NAMESPACE::Config::GetProcessorFromConfigs(
					Context,
					Config,
					StringCast<ANSICHAR>(InSourceColorSpace.GetData()).Get(),
					Context,
					IOpenColorIOWrapperModule::Get().GetEngineBuiltInConfig().Pimpl->Config,
					AnsiWorkingColorSpaceName.Get());
			}
			else
			{
				Pimpl->Processor = Config->getProcessor(
					Context,
					StringCast<ANSICHAR>(InSourceColorSpace.GetData()).Get(),
					StringCast<ANSICHAR>(InDestinationColorSpace.GetData()).Get()
				);
			}
		}
	OCIO_EXCEPTION_HANDLING_CATCH(Log, TEXT("Failed to create processor for [%s, %s]. Error message: %s"), InSourceColorSpace.GetData(), InDestinationColorSpace.GetData());
}

FOpenColorIOWrapperProcessor::FOpenColorIOWrapperProcessor(
	const FOpenColorIOWrapperConfig* InConfig,
	FStringView InSourceColorSpace,
	FStringView InDisplay,
	FStringView InView,
	bool bInverseDirection,
	const TMap<FString, FString>& InContextKeyValues)
	: Pimpl(MakePimpl<FOpenColorIOProcessorPimpl, EPimplPtrMode::DeepCopy>())
{
	OCIO_EXCEPTION_HANDLING_TRY();
		if (InConfig != nullptr && InConfig->IsValid())
		{
			const OCIO_NAMESPACE::ConstConfigRcPtr& Config = InConfig->Pimpl->Config;
			OCIO_NAMESPACE::ContextRcPtr Context = Config->getCurrentContext()->createEditableCopy();

			for (const TPair<FString, FString>& KeyValue : InContextKeyValues)
			{
				Context->setStringVar(TCHAR_TO_ANSI(*KeyValue.Key), TCHAR_TO_ANSI(*KeyValue.Value));
			}

			const TCHAR* WCSColorSpaceName = OpenColorIOWrapper::GetWorkingColorSpaceName();
			const TUniquePtr<ANSICHAR[]> AnsiWorkingColorSpaceName = OpenColorIOWrapper::MakeAnsiString(WCSColorSpaceName);

			if (InSourceColorSpace == WCSColorSpaceName)
			{
				Pimpl->Processor = OCIO_NAMESPACE::Config::GetProcessorFromConfigs(
					Context,
					IOpenColorIOWrapperModule::Get().GetEngineBuiltInConfig().Pimpl->Config,
					AnsiWorkingColorSpaceName.Get(),
					Context,
					Config,
					StringCast<ANSICHAR>(InDisplay.GetData()).Get(),
					StringCast<ANSICHAR>(InView.GetData()).Get(),
					static_cast<OCIO_NAMESPACE::TransformDirection>(bInverseDirection));
			}
			else
			{
				Pimpl->Processor = Config->getProcessor(
					Context,
					StringCast<ANSICHAR>(InSourceColorSpace.GetData()).Get(),
					StringCast<ANSICHAR>(InDisplay.GetData()).Get(),
					StringCast<ANSICHAR>(InView.GetData()).Get(),
					static_cast<OCIO_NAMESPACE::TransformDirection>(bInverseDirection)
				);
			}
		}
	OCIO_EXCEPTION_HANDLING_CATCH(Log, TEXT("Failed to create processor for [%s, %s, %s, %s]. Error message: %s"), InSourceColorSpace.GetData(), InDisplay.GetData(), InView.GetData(), (bInverseDirection ? TEXT("Inverse") : TEXT("Forward")));
}

FOpenColorIOWrapperProcessor::FOpenColorIOWrapperProcessor(const FOpenColorIOWrapperConfig* InConfig, FStringView InNamedTransform, bool bInverseDirection, const TMap<FString, FString>& InContextKeyValues)
	: Pimpl(MakePimpl<FOpenColorIOProcessorPimpl, EPimplPtrMode::DeepCopy>())
{
	OCIO_EXCEPTION_HANDLING_TRY();
		if (InConfig != nullptr && InConfig->IsValid())
		{
			const OCIO_NAMESPACE::ConstConfigRcPtr& Config = InConfig->Pimpl->Config;
			OCIO_NAMESPACE::ContextRcPtr Context = Config->getCurrentContext()->createEditableCopy();

			for (const TPair<FString, FString>& KeyValue : InContextKeyValues)
			{
				Context->setStringVar(TCHAR_TO_ANSI(*KeyValue.Key), TCHAR_TO_ANSI(*KeyValue.Value));
			}

			Pimpl->Processor = Config->getProcessor(
				Context,
				StringCast<ANSICHAR>(InNamedTransform.GetData()).Get(),
				static_cast<OCIO_NAMESPACE::TransformDirection>(bInverseDirection)
			);
		}
	OCIO_EXCEPTION_HANDLING_CATCH(Log, TEXT("Failed to create processor for [%s]. Error message: %s"), InNamedTransform.GetData());
}

FOpenColorIOWrapperProcessor FOpenColorIOWrapperProcessor::CreateTransformToWorkingColorSpace(const FOpenColorIOWrapperSourceColorSettings& InColorSettings)
{
	FOpenColorIOWrapperConfig& BuiltInConfig = IOpenColorIOWrapperModule::Get().GetEngineBuiltInConfig();

	const FString TransformName = GetTransformToWorkingColorSpaceName(InColorSettings);

	using namespace OCIO_NAMESPACE;
	using namespace UE::Color;

	static FCriticalSection BuiltInConfigCriticalSection;
	FScopeLock Lock(&BuiltInConfigCriticalSection);

	if (BuiltInConfig.Pimpl->Config->getNamedTransform(StringCast<ANSICHAR>(*TransformName).Get()) == nullptr)
	{
		GroupTransformRcPtr TransformToWCS = GroupTransform::Create();
		NamedTransformRcPtr ParentTransform = NamedTransform::Create();
		ParentTransform->setName(StringCast<ANSICHAR>(*TransformName).Get());

		switch (InColorSettings.EncodingOverride)
		{
		case EEncoding::None:
		case EEncoding::Linear:
			ParentTransform->setEncoding("scene-linear");
			break;
		case EEncoding::sRGB:
		{
			ExponentWithLinearTransformRcPtr ChildTransform = ExponentWithLinearTransform::Create();
			ChildTransform->setGamma({ 2.4, 2.4, 2.4, 1.0 });
			ChildTransform->setOffset({ 0.055, 0.055, 0.055, 0.0 });

			TransformToWCS->appendTransform(ChildTransform);
			ParentTransform->setEncoding("sdr-video");
		}
		break;
		case EEncoding::ST2084:
		{
			/**
			 * Spec-literal ST2084 / PQ: output is display-referred linear light in cd/m2 (nits).
			 * OCIO's built-in CURVE emits nits/100, so we rescale by 100 to recover full nits.
			 */
			BuiltinTransformRcPtr ChildTransform = BuiltinTransform::Create();
			ChildTransform->setStyle("CURVE - ST-2084_to_LINEAR");
			TransformToWCS->appendTransform(ChildTransform);

			MatrixTransformRcPtr RescaleTransform = MatrixTransform::Create();
			FMatrix44d ScaleMatrix = FMatrix44d::Identity.ApplyScale(100.0);
			RescaleTransform->setMatrix(&ScaleMatrix.M[0][0]);

			TransformToWCS->appendTransform(RescaleTransform);
			ParentTransform->setEncoding("hdr-video");
		}
		break;
		case EEncoding::Gamma22:
		{
			ExponentTransformRcPtr ChildTransform = ExponentTransform::Create();
			ChildTransform->setValue({ 2.2, 2.2, 2.2, 1.0 });
			ChildTransform->setNegativeStyle(NEGATIVE_PASS_THRU);

			TransformToWCS->appendTransform(ChildTransform);
			ParentTransform->setEncoding("sdr-video");
		}
		break;
		case EEncoding::BT1886:
		{
			ExponentTransformRcPtr ChildTransform = ExponentTransform::Create();
			ChildTransform->setValue({ 2.4, 2.4, 2.4, 1.0 });
			ChildTransform->setNegativeStyle(NEGATIVE_PASS_THRU);

			TransformToWCS->appendTransform(ChildTransform);
			ParentTransform->setEncoding("sdr-video");
		}
		break;
		case EEncoding::Gamma26:
		{
			ExponentTransformRcPtr ChildTransform = ExponentTransform::Create();
			ChildTransform->setValue({ 2.6, 2.6, 2.6, 1.0 });
			ChildTransform->setNegativeStyle(NEGATIVE_PASS_THRU);

			TransformToWCS->appendTransform(ChildTransform);
			ParentTransform->setEncoding("sdr-video");
		}
		break;
		case EEncoding::Cineon:
		{
			const double BlackOffset = FGenericPlatformMath::Pow(10.0, (95.0 - 685.0) / 300.0);
			const double LinSideSlope = 1.0 - BlackOffset;
			const double LinSideOffset = BlackOffset;
			static constexpr double LogSideSlope = 300.0 / 1023.0;
			static constexpr double LogSideOffset = 685.0 / 1023.0;
			static constexpr double Base = 10.;

			LogAffineTransformRcPtr ChildTransform = LogAffineTransform::Create();
			ChildTransform->setLinSideSlopeValue({ LinSideSlope, LinSideSlope,LinSideSlope });
			ChildTransform->setLinSideOffsetValue({ LinSideOffset, LinSideOffset,LinSideOffset });
			ChildTransform->setLogSideSlopeValue({ LogSideSlope, LogSideSlope, LogSideSlope });
			ChildTransform->setLogSideOffsetValue({ LogSideOffset, LogSideOffset, LogSideOffset });
			ChildTransform->setBase(Base);
			ChildTransform->setDirection(TRANSFORM_DIR_INVERSE);

			TransformToWCS->appendTransform(ChildTransform);
			ParentTransform->setEncoding("log");
		}
		break;
		case EEncoding::REDLog:
		{
			const double BlackOffset = FGenericPlatformMath::Pow(10.0, (0.0 - 1023.0) / 511.0);
			const double LinSideSlope = 1.0 - BlackOffset;
			const double LinSideOffset = BlackOffset;
			static constexpr double LogSideSlope = 511.0 / 1023.0;
			static constexpr double LogSideOffset = 1.0;
			static constexpr double Base = 10.;

			LogAffineTransformRcPtr ChildTransform = LogAffineTransform::Create();
			ChildTransform->setLinSideSlopeValue({ LinSideSlope, LinSideSlope,LinSideSlope });
			ChildTransform->setLinSideOffsetValue({ LinSideOffset, LinSideOffset,LinSideOffset });
			ChildTransform->setLogSideSlopeValue({ LogSideSlope, LogSideSlope, LogSideSlope });
			ChildTransform->setLogSideOffsetValue({ LogSideOffset, LogSideOffset, LogSideOffset });
			ChildTransform->setBase(Base);
			ChildTransform->setDirection(TRANSFORM_DIR_INVERSE);

			TransformToWCS->appendTransform(ChildTransform);
			ParentTransform->setEncoding("log");
		}
		break;
		case EEncoding::REDLog3G10:
		{
			static constexpr double LinSideSlope = 155.975327;
			static constexpr double LinSideOffset = 0.01 * LinSideSlope + 1.0;
			static constexpr double LogSideSlope = 0.224282;
			static constexpr double LogSideOffset = 0.0;
			static constexpr double LinSideBreak = -0.01;
			static constexpr double Base = 10.;

			LogCameraTransformRcPtr ChildTransform = LogCameraTransform::Create({ LinSideBreak, LinSideBreak,LinSideBreak });
			ChildTransform->setLinSideSlopeValue({ LinSideSlope, LinSideSlope,LinSideSlope });
			ChildTransform->setLinSideOffsetValue({ LinSideOffset, LinSideOffset,LinSideOffset });
			ChildTransform->setLogSideSlopeValue({ LogSideSlope, LogSideSlope, LogSideSlope });
			ChildTransform->setLogSideOffsetValue({ LogSideOffset, LogSideOffset, LogSideOffset });
			ChildTransform->setBase(Base);
			ChildTransform->setDirection(TRANSFORM_DIR_INVERSE);

			TransformToWCS->appendTransform(ChildTransform);
			ParentTransform->setEncoding("log");
		}
		break;
		case EEncoding::SLog1:
		{
			static constexpr double LinSideSlope = 1.0 / 0.9;
			static constexpr double LinSideOffset = 0.037584;
			static constexpr double LogSideSlope = 0.432699 * 219.0 * 4.0 / 1023.0;
			static constexpr double LogSideOffset = ((0.616596 + 0.03) * 219.0 + 16.0) * 4.0 / 1023.0;
			static constexpr double Base = 10.;

			LogAffineTransformRcPtr ChildTransform = LogAffineTransform::Create();
			ChildTransform->setLinSideSlopeValue({ LinSideSlope, LinSideSlope,LinSideSlope });
			ChildTransform->setLinSideOffsetValue({ LinSideOffset, LinSideOffset,LinSideOffset });
			ChildTransform->setLogSideSlopeValue({ LogSideSlope, LogSideSlope, LogSideSlope });
			ChildTransform->setLogSideOffsetValue({ LogSideOffset, LogSideOffset, LogSideOffset });
			ChildTransform->setBase(Base);
			ChildTransform->setDirection(TRANSFORM_DIR_INVERSE);

			TransformToWCS->appendTransform(ChildTransform);
			ParentTransform->setEncoding("log");
		}
		break;
		case EEncoding::SLog2:
		{
			static constexpr double LinSideSlope = 155.0 / 197.1;
			static constexpr double LinSideOffset = 0.037584;
			static constexpr double LogSideSlope = 876.0 * 0.432699 / 1023.0;
			static constexpr double LogSideOffset = (64.0 + 876.0 * 0.646596) / 1023.0;
			static constexpr double LinSideBreak = 0.0;
			static constexpr double LinearSlope = 876.0 * (3.53881278538813f / 0.9) / 1023.0;
			static constexpr double Base = 10.;

			LogCameraTransformRcPtr ChildTransform = LogCameraTransform::Create({ LinSideBreak, LinSideBreak,LinSideBreak });
			ChildTransform->setLinSideSlopeValue({ LinSideSlope, LinSideSlope,LinSideSlope });
			ChildTransform->setLinSideOffsetValue({ LinSideOffset, LinSideOffset,LinSideOffset });
			ChildTransform->setLogSideSlopeValue({ LogSideSlope, LogSideSlope, LogSideSlope });
			ChildTransform->setLogSideOffsetValue({ LogSideOffset, LogSideOffset, LogSideOffset });
			ChildTransform->setLinearSlopeValue({ LinearSlope, LinearSlope, LinearSlope });
			ChildTransform->setBase(Base);
			ChildTransform->setDirection(TRANSFORM_DIR_INVERSE);

			TransformToWCS->appendTransform(ChildTransform);
			ParentTransform->setEncoding("log");
		}
		break;
		case EEncoding::SLog3:
		{
			static constexpr double LinSideSlope = 5.26315789473684;
			static constexpr double LinSideOffset = 0.0526315789473684;
			static constexpr double LogSideSlope = 0.255620723362659;
			static constexpr double LogSideOffset = 0.410557184750733;
			static constexpr double LinSideBreak = 0.01125;
			static constexpr double LinearSlope = 6.62194371177582;
			static constexpr double Base = 10.;

			LogCameraTransformRcPtr ChildTransform = LogCameraTransform::Create({ LinSideBreak, LinSideBreak,LinSideBreak });
			ChildTransform->setLinSideSlopeValue({ LinSideSlope, LinSideSlope,LinSideSlope });
			ChildTransform->setLinSideOffsetValue({ LinSideOffset, LinSideOffset,LinSideOffset });
			ChildTransform->setLogSideSlopeValue({ LogSideSlope, LogSideSlope, LogSideSlope });
			ChildTransform->setLogSideOffsetValue({ LogSideOffset, LogSideOffset, LogSideOffset });
			ChildTransform->setLinearSlopeValue({ LinearSlope, LinearSlope, LinearSlope });
			ChildTransform->setBase(Base);
			ChildTransform->setDirection(TRANSFORM_DIR_INVERSE);

			TransformToWCS->appendTransform(ChildTransform);
			ParentTransform->setEncoding("log");
		}
		break;
		case EEncoding::AlexaV3LogC:
		{
			static constexpr double LinSideSlope = 5.55555555555556;
			static constexpr double LinSideOffset = 0.0522722750251688;
			static constexpr double LogSideSlope = 0.247189638318671;
			static constexpr double LogSideOffset = 0.385536998692443;
			static constexpr double LinSideBreak = 0.0105909904954696;
			static constexpr double Base = 10.;

			LogCameraTransformRcPtr ChildTransform = LogCameraTransform::Create({ LinSideBreak, LinSideBreak,LinSideBreak });
			ChildTransform->setLinSideSlopeValue({ LinSideSlope, LinSideSlope,LinSideSlope });
			ChildTransform->setLinSideOffsetValue({ LinSideOffset, LinSideOffset,LinSideOffset });
			ChildTransform->setLogSideSlopeValue({ LogSideSlope, LogSideSlope, LogSideSlope });
			ChildTransform->setLogSideOffsetValue({ LogSideOffset, LogSideOffset, LogSideOffset });
			ChildTransform->setBase(Base);
			ChildTransform->setDirection(TRANSFORM_DIR_INVERSE);

			TransformToWCS->appendTransform(ChildTransform);
			ParentTransform->setEncoding("log");
		}
		break;
		case EEncoding::CanonLog:
		{
			static constexpr double LinSideSlope = 10.1596;
			static constexpr double LinSideOffset = 1.0;
			static constexpr double LogSideSlope = 0.529136;
			static constexpr double LogSideOffset = 0.0730597;
			static constexpr double LinSideBreak = 0.0;
			static constexpr double Base = 10.;

			LogAffineTransformRcPtr ChildTransform = LogAffineTransform::Create();
			ChildTransform->setLinSideSlopeValue({ LinSideSlope, LinSideSlope,LinSideSlope });
			ChildTransform->setLinSideOffsetValue({ LinSideOffset, LinSideOffset,LinSideOffset });
			ChildTransform->setLogSideSlopeValue({ LogSideSlope, LogSideSlope, LogSideSlope });
			ChildTransform->setLogSideOffsetValue({ LogSideOffset, LogSideOffset, LogSideOffset });
			ChildTransform->setBase(Base);
			ChildTransform->setDirection(TRANSFORM_DIR_INVERSE);

			TransformToWCS->appendTransform(ChildTransform);
			ParentTransform->setEncoding("log");
		}
		break;
		case EEncoding::ProTune:
		{
			static constexpr double LinSideSlope = 112.0;
			static constexpr double LinSideOffset = 1.0f;
			const double LogSideSlope = 1.0 / FGenericPlatformMath::Loge(113.0);

			LogAffineTransformRcPtr ChildTransform = LogAffineTransform::Create();
			ChildTransform->setLinSideSlopeValue({ LinSideSlope, LinSideSlope,LinSideSlope });
			ChildTransform->setLinSideOffsetValue({ LinSideOffset, LinSideOffset,LinSideOffset });
			ChildTransform->setLogSideSlopeValue({ LogSideSlope, LogSideSlope, LogSideSlope });
			ChildTransform->setBase(UE_DOUBLE_EULERS_NUMBER);
			ChildTransform->setDirection(TRANSFORM_DIR_INVERSE);

			TransformToWCS->appendTransform(ChildTransform);
			ParentTransform->setEncoding("log");
		}
		break;
		case EEncoding::VLog:
		{
			static constexpr double LinSideSlope = 1.0;
			static constexpr double LinSideOffset = 0.00873;
			static constexpr double LogSideSlope = 0.241514;
			static constexpr double LogSideOffset = 0.598206;
			static constexpr double LinSideBreak = 0.01;
			static constexpr double Base = 10.;
			// Note: this is not in the studio config
			// static constexpr double LinearSlope = 5.6;

			LogCameraTransformRcPtr ChildTransform = LogCameraTransform::Create({ LinSideBreak, LinSideBreak,LinSideBreak });
			ChildTransform->setLinSideSlopeValue({ LinSideSlope, LinSideSlope,LinSideSlope });
			ChildTransform->setLinSideOffsetValue({ LinSideOffset, LinSideOffset,LinSideOffset });
			ChildTransform->setLogSideSlopeValue({ LogSideSlope, LogSideSlope, LogSideSlope });
			ChildTransform->setLogSideOffsetValue({ LogSideOffset, LogSideOffset, LogSideOffset });
			//ChildTransform->setLinearSlopeValue({ LinearSlope, LinearSlope, LinearSlope });
			ChildTransform->setBase(Base);
			ChildTransform->setDirection(TRANSFORM_DIR_INVERSE);

			TransformToWCS->appendTransform(ChildTransform);
			ParentTransform->setEncoding("log");
		}
		break;
		case EEncoding::HLG:
		{
			/**
			 * Spec-literal BT.2100 inverse OETF via OCIO's built-in CURVE - HLG-OETF-INVERSE,
			 * which emits values in [0, 3] (i.e. 3 * BT.2100 literal). Post-multiply by 1/3
			 * to land at the BT.2100 literal [0, 1] form.
			 */
			BuiltinTransformRcPtr ChildTransform = BuiltinTransform::Create();
			ChildTransform->setStyle("CURVE - HLG-OETF-INVERSE");
			TransformToWCS->appendTransform(ChildTransform);

			MatrixTransformRcPtr MatrixTransform = MatrixTransform::Create();
			FMatrix44d ScaleMat = FMatrix44d::Identity * (1.0 / 3.0);
			ScaleMat.M[3][3] = 1.0;
			MatrixTransform->setMatrix(&ScaleMat.M[0][0]);
			TransformToWCS->appendTransform(MatrixTransform);
			ParentTransform->setEncoding("hdr-video");
		}
		break;

		default:
			checkNoEntry();
			break;
		}

		TOptional<FColorSpace> SourceColorSpace;

		if (InColorSettings.ColorSpaceOverride.IsSet())
		{
			const TStaticArray<FVector2d, 4>& SourceChromaticities = InColorSettings.ColorSpaceOverride.GetValue();
			SourceColorSpace = FColorSpace(
				SourceChromaticities[0],
				SourceChromaticities[1],
				SourceChromaticities[2],
				SourceChromaticities[3]
			);
		}
		else if (InColorSettings.ColorSpace != EColorSpace::None)
		{
			SourceColorSpace = FColorSpace(InColorSettings.ColorSpace);
		}

		if (SourceColorSpace.IsSet())
		{
			if (!SourceColorSpace.GetValue().Equals(FColorSpace::GetWorking()))
			{
				const EChromaticAdaptationMethod ChromaticAdaptation = static_cast<EChromaticAdaptationMethod>(InColorSettings.ChromaticAdaptationMethod);
				const FMatrix44d ToWorkingMat = Transpose<double>(FColorSpaceTransform(SourceColorSpace.GetValue(), FColorSpace::GetWorking(), ChromaticAdaptation));
				MatrixTransformRcPtr MatrixTransform = MatrixTransform::Create();
				MatrixTransform->setMatrix(&ToWorkingMat.M[0][0]);
				
				TransformToWCS->appendTransform(MatrixTransform);
			}
		}

		ParentTransform->setTransform(TransformToWCS, TRANSFORM_DIR_FORWARD);

		// Update builtin config
		ConstConfigRcPtr& CurrentConfig = BuiltInConfig.Pimpl->Config;
		ConfigRcPtr NewConfig = CurrentConfig->createEditableCopy();
		NewConfig->addNamedTransform(ParentTransform);
		CurrentConfig = NewConfig;
	}

	return FOpenColorIOWrapperProcessor(&BuiltInConfig, TransformName);
}

bool FOpenColorIOWrapperProcessor::IsValid() const
{
	return Pimpl->Processor != nullptr;
}

FString FOpenColorIOWrapperProcessor::GetCacheID() const
{
	OCIO_EXCEPTION_HANDLING_TRY();
		if (IsValid())
		{
			return StringCast<TCHAR>(Pimpl->Processor->getCacheID()).Get();
		}
	OCIO_EXCEPTION_HANDLING_CATCH_ERROR()

	return {};
}

FString FOpenColorIOWrapperProcessor::GetTransformToWorkingColorSpaceName(const FOpenColorIOWrapperSourceColorSettings& InSourceColorSettings)
{
	const uint32 SettingsId = (uint32)InSourceColorSettings.EncodingOverride | (uint32)InSourceColorSettings.ColorSpace << 8u | (uint32)InSourceColorSettings.ChromaticAdaptationMethod << 16u;
	FString TransformName = FString::Printf(TEXT("UE_%u"), SettingsId);

	if (InSourceColorSettings.ColorSpaceOverride.IsSet())
	{
		uint32 SrcChromaticityHash = 0;
		SrcChromaticityHash ^= GetTypeHash(InSourceColorSettings.ColorSpaceOverride.GetValue()[0]);
		SrcChromaticityHash ^= GetTypeHash(InSourceColorSettings.ColorSpaceOverride.GetValue()[1]);
		SrcChromaticityHash ^= GetTypeHash(InSourceColorSettings.ColorSpaceOverride.GetValue()[2]);
		SrcChromaticityHash ^= GetTypeHash(InSourceColorSettings.ColorSpaceOverride.GetValue()[3]);
		TransformName += FString::Printf(TEXT("_%u"), SrcChromaticityHash);
	}

	return TransformName;
}

bool FOpenColorIOWrapperProcessor::TransformColor(FLinearColor& InOutColor) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FOpenColorIOWrapperProcessor::TransformColor);
	OCIO_EXCEPTION_HANDLING_TRY();

		if (IsValid())
		{
			// Apply the main color transformation
			OCIO_NAMESPACE::ConstCPUProcessorRcPtr CPUProcessor = Pimpl->Processor->getDefaultCPUProcessor();
			CPUProcessor->applyRGBA(&InOutColor.R);

			return true;
		}
	OCIO_EXCEPTION_HANDLING_CATCH(Error, TEXT("Failed to transform color. Error message: %s"));

	return false;
}

bool FOpenColorIOWrapperProcessor::TransformImage(const FImageView& InOutImage) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FOpenColorIOWrapperProcessor::TransformImage);
	OCIO_EXCEPTION_HANDLING_TRY();
		using namespace OCIO_NAMESPACE;

		if (IsValid())
		{
			if (IsImageFormatSupported(InOutImage))
			{
				const BitDepth BitDepth = GetImageDesc(InOutImage).getBitDepth();

				// Primary transform processor
				ConstCPUProcessorRcPtr CPUProcessor = Pimpl->Processor->getOptimizedCPUProcessor(BitDepth, BitDepth, OPTIMIZATION_DEFAULT);

				// Apply parallelized color transformation (when it isn't a no-op)
				if (!CPUProcessor->isNoOp())
				{
					FImageCore::ImageParallelFor(TEXT("FOpenColorIOWrapperProcessor.TransformImage.PF"), InOutImage, [&](FImageView& ImagePart,int64 RowY)
						{
							OCIO_NAMESPACE::PackedImageDesc ImagePartDesc = GetImageDesc(ImagePart);

							// Apply the main color transformation
							CPUProcessor->apply(ImagePartDesc);
						});
				}

				return true;
			}
			else
			{
				UE_LOGF(LogOpenColorIOWrapper, Warning, "Unsupported texture format.");
			}
		}
	OCIO_EXCEPTION_HANDLING_CATCH(Error, TEXT("Failed to transform image. Error message: %s"));

	return false;
}

bool FOpenColorIOWrapperProcessor::TransformImage(const FImageView& SrcImage, const FImageView& DestImage) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FOpenColorIOWrapperProcessor::TransformImage);
	OCIO_EXCEPTION_HANDLING_TRY();
		using namespace OCIO_NAMESPACE;

		if (IsValid())
		{
			if (IsImageFormatSupported(SrcImage) && IsImageFormatSupported(DestImage))
			{
				PackedImageDesc SrcImageDesc = GetImageDesc(SrcImage);
				PackedImageDesc DestImageDesc = GetImageDesc(DestImage);
				BitDepth SrcBitDepth = SrcImageDesc.getBitDepth();
				BitDepth DestBitDepth = DestImageDesc.getBitDepth();

				// Apply the main color transformation
				ConstCPUProcessorRcPtr CPUProcessor = Pimpl->Processor->getOptimizedCPUProcessor(SrcBitDepth, DestBitDepth, OPTIMIZATION_DEFAULT);
				CPUProcessor->apply(SrcImageDesc, DestImageDesc);

				return true;
			}
			else
			{
				UE_LOGF(LogOpenColorIOWrapper, Warning, "Unsupported texture format(s).");
			}
		}
	OCIO_EXCEPTION_HANDLING_CATCH(Error, TEXT("Failed to transform image. Error message: %s"));

	return false;
}



FOpenColorIOWrapperGPUProcessor::FOpenColorIOWrapperGPUProcessor(FOpenColorIOWrapperProcessor InProcessor, bool bUseLegacy)
	: ParentProcessor(MoveTemp(InProcessor))
	, GPUPimpl(MakePimpl<FOpenColorIOGPUProcessorPimpl, EPimplPtrMode::DeepCopy>())
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FOpenColorIOWrapperGPUProcessor::FOpenColorIOWrapperGPUProcessor);
	OCIO_EXCEPTION_HANDLING_TRY();
		using namespace OCIO_NAMESPACE;

		if (ParentProcessor.IsValid())
		{
			GpuShaderDescRcPtr ShaderDescription = GpuShaderDesc::CreateShaderDesc();
			ShaderDescription->setLanguage(GPU_LANGUAGE_HLSL_SM_5_0);
			ShaderDescription->setFunctionName(StringCast<ANSICHAR>(OpenColorIOWrapper::GetShaderFunctionName()).Get());
			ShaderDescription->setResourcePrefix("Ocio");
			ShaderDescription->setAllowTexture1D(false);

			ConstGPUProcessorRcPtr GPUProcessor = nullptr;
			OptimizationFlags OptFlags = FOpenColorIOProcessorPimpl::GetOptimizationFlags();

			if (bUseLegacy)
			{
				unsigned int EdgeLength = static_cast<unsigned int>(OpenColorIOWrapper::Legacy3dEdgeLength);
				GPUProcessor = ParentProcessor.Pimpl->Processor->getOptimizedLegacyGPUProcessor(OptFlags, EdgeLength);
			}
			else
			{
				GPUProcessor = ParentProcessor.Pimpl->Processor->getOptimizedGPUProcessor(FOpenColorIOProcessorPimpl::GetOptimizationFlags());
			}
			GPUProcessor->extractGpuShaderInfo(ShaderDescription);

			GPUPimpl->Processor = GPUProcessor;
			GPUPimpl->ShaderDescription = ShaderDescription;
		}
	OCIO_EXCEPTION_HANDLING_CATCH(Log, TEXT("Failed to fetch shader info for color transform. Error message: %s"));
}

bool FOpenColorIOWrapperGPUProcessor::IsValid() const
{
	return ParentProcessor.IsValid() && GPUPimpl->Processor != nullptr && GPUPimpl->ShaderDescription != nullptr;
}

bool FOpenColorIOWrapperGPUProcessor::GetShader(FString& OutShaderCacheID, FString& OutShaderCode) const
{
	if (IsValid())
	{
		ensureMsgf(GPUPimpl->ShaderDescription->getNumDynamicProperties() == 0, TEXT("We do not currently support dynamic properties."));

		OutShaderCacheID = StringCast<TCHAR>(GPUPimpl->ShaderDescription->getCacheID()).Get();
		OutShaderCode = StringCast<TCHAR>(GPUPimpl->ShaderDescription->getShaderText()).Get();

		return true;
	}

	return false;
}

uint32 FOpenColorIOWrapperGPUProcessor::GetNum3DTextures() const
{
	if (IsValid())
	{
		return GPUPimpl->ShaderDescription->getNum3DTextures();
	}

	return 0;
}

bool FOpenColorIOWrapperGPUProcessor::Get3DTexture(uint32 InIndex, FName& OutName, uint32& OutEdgeLength, TextureFilter& OutTextureFilter, const float*& OutData) const
{
	ensure(InIndex < GetNum3DTextures());

	OCIO_EXCEPTION_HANDLING_TRY();
		if (IsValid())
		{
			const ANSICHAR* TextureName = nullptr;
			const ANSICHAR* SamplerName = nullptr;
			OCIO_NAMESPACE::Interpolation Interpolation = OCIO_NAMESPACE::INTERP_TETRAHEDRAL;

			// Read texture information
			GPUPimpl->ShaderDescription->get3DTexture(InIndex, TextureName, SamplerName, OutEdgeLength, Interpolation);

			// Read texture data
			OutData = 0x0;
			GPUPimpl->ShaderDescription->get3DTextureValues(InIndex, OutData);

			OutName = FName(TextureName);

			OutTextureFilter = TF_Bilinear;
			if (Interpolation == OCIO_NAMESPACE::Interpolation::INTERP_NEAREST || Interpolation == OCIO_NAMESPACE::Interpolation::INTERP_TETRAHEDRAL)
			{
				OutTextureFilter = TF_Nearest;
			}

			return TextureName && OutEdgeLength > 0 && OutData;
		}
	OCIO_EXCEPTION_HANDLING_CATCH(Log, TEXT("Failed to fetch 3d texture(s) info for color transform. Error message: %s"));

	return false;
}
uint32 FOpenColorIOWrapperGPUProcessor::GetNumTextures() const
{
	if (IsValid())
	{
		return GPUPimpl->ShaderDescription->getNumTextures();  //noexcept
	}

	return 0;
}

bool FOpenColorIOWrapperGPUProcessor::GetTexture(uint32 InIndex, FName& OutName, uint32& OutWidth, uint32& OutHeight, TextureFilter& OutTextureFilter, bool& bOutRedChannelOnly, const float*& OutData) const
{
	ensure(InIndex < GetNumTextures());

	OCIO_EXCEPTION_HANDLING_TRY();
		if (IsValid())
		{
			const ANSICHAR* TextureName = nullptr;
			const ANSICHAR* SamplerName = nullptr;
			OCIO_NAMESPACE::GpuShaderDesc::TextureType Channel = OCIO_NAMESPACE::GpuShaderDesc::TEXTURE_RGB_CHANNEL;
			OCIO_NAMESPACE::GpuShaderDesc::TextureDimensions Dimensions = OCIO_NAMESPACE::GpuShaderDesc::TextureDimensions::TEXTURE_2D;
			OCIO_NAMESPACE::Interpolation Interpolation = OCIO_NAMESPACE::Interpolation::INTERP_LINEAR;

			// Read texture information
			GPUPimpl->ShaderDescription->getTexture(InIndex, TextureName, SamplerName, OutWidth, OutHeight, Channel, Dimensions, Interpolation);
			
			// 1D LUT textures will always be 2D because we disallow 1d resources.
			check(Dimensions == OCIO_NAMESPACE::GpuShaderDesc::TextureDimensions::TEXTURE_2D);

			// Read texture data
			OutData = 0x0;
			GPUPimpl->ShaderDescription->getTextureValues(InIndex, OutData);

			OutName = FName(TextureName);
			OutTextureFilter = Interpolation == OCIO_NAMESPACE::Interpolation::INTERP_NEAREST ? TF_Nearest : TF_Bilinear;
			bOutRedChannelOnly = Channel == OCIO_NAMESPACE::GpuShaderCreator::TEXTURE_RED_CHANNEL;

			return TextureName && OutWidth > 0 && OutHeight > 0 && OutData;
		}
	OCIO_EXCEPTION_HANDLING_CATCH(Error, TEXT("Failed to fetch texture(s) info for color transform. Error message: %s"));

	return false;
}

FString FOpenColorIOWrapperGPUProcessor::GetCacheID() const
{
	OCIO_EXCEPTION_HANDLING_TRY();
		if (IsValid())
		{
			return StringCast<TCHAR>(GPUPimpl->Processor->getCacheID()).Get();
		}
	OCIO_EXCEPTION_HANDLING_CATCH_ERROR()

	return {};
}

#undef LOCTEXT_NAMESPACE
#endif //WITH_OCIO
