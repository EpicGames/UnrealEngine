// Copyright Epic Games, Inc. All Rights Reserved.

#include "OpenColorIOWrapperModule.h"

#include "OpenColorIOWrapper.h"


DEFINE_LOG_CATEGORY(LogOpenColorIOWrapper);

#define OPENCOLORIOWRAPPER_MODULE_NAME "OpenColorIOWrapper"
#define LOCTEXT_NAMESPACE "OpenColorIOWrapperModule"

class FOpenColorIOWrapperModule : public IOpenColorIOWrapperModule
{
public:
	//~ Begin IModuleInterface interface
	virtual void StartupModule() override
	{
	}

	virtual void ShutdownModule() override
	{
#if WITH_OCIO
		EngineBuiltInConfig.Reset();
#endif //WITH_OCIO
	}
	//~ End IModuleInterface interface

#if WITH_OCIO
	//~ Begin IOpenColorIOWrapperModule interface
	virtual FOpenColorIOWrapperEngineConfig& GetEngineBuiltInConfig() override
	{
		FScopeLock Lock(&ConfigCriticalSection);

		if (!EngineBuiltInConfig)
		{
			EngineBuiltInConfig = MakeUnique<FOpenColorIOWrapperEngineConfig>();
		}

		return *EngineBuiltInConfig;
	}
	//~ End IOpenColorIOWrapperModule interface
private:

	// Critical section for the engine config getter.
	FCriticalSection ConfigCriticalSection;

	// Global engine config using the built-in studio config, lazily allocated.
	TUniquePtr<FOpenColorIOWrapperEngineConfig> EngineBuiltInConfig;
#endif //WITH_OCIO
};

IMPLEMENT_MODULE(FOpenColorIOWrapperModule, OpenColorIOWrapper);

#undef LOCTEXT_NAMESPACE
