// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Misc/AutomationTest.h"

#if WITH_DEV_AUTOMATION_TESTS && WITH_OCIO

#include "ColorManagement/ColorSpace.h"
#include "Logging/LogMacros.h"
#include "Math/UnrealMathUtility.h"
#include "OpenColorIOWrapper.h"
#include "ColorManagement/TransferFunctions.h"

DEFINE_LOG_CATEGORY_STATIC(LogUnrealOpenColorIOTest, Log, All);

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FOpenColorIOTransferFunctionsTest, "System.OpenColorIO.DecodeToWorkingColorSpace", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FOpenColorIOTransferFunctionsTest::RunTest(const FString& Parameters)
{
	using namespace UE::Color;

	bool bSuccess = true;
	const FLinearColor TestColor = FLinearColor(0.9f, 0.5f, 0.2f, 1.0f);

	for (uint8 TestEncoding = static_cast<uint8>(EEncoding::None); TestEncoding < static_cast<uint8>(EEncoding::Max); ++TestEncoding)
	{
		FLinearColor Expected = UE::Color::Decode(static_cast<EEncoding>(TestEncoding), TestColor);

		FOpenColorIOWrapperSourceColorSettings TestSettings;
		TestSettings.EncodingOverride = static_cast<EEncoding>(TestEncoding);

		FOpenColorIOWrapperProcessor Processor = FOpenColorIOWrapperProcessor::CreateTransformToWorkingColorSpace(TestSettings);

		FLinearColor Actual = TestColor;
		Processor.TransformColor(Actual);

		// Note: We make the tolerance relative to the values themselves to account for larger values in PQ.
		const float Tolerance = UE_KINDA_SMALL_NUMBER * 0.5f * (Actual.R + Expected.R);

		if (!Actual.Equals(Expected, Tolerance))
		{
			const FString TestNameToPrint = FString::Printf(TEXT("OpenColorIO: %u:%u"), (uint32)TestSettings.EncodingOverride, (uint32)TestSettings.ColorSpace);
			AddError(FString::Printf(TEXT("Expected '%s' to be %s, but it was %s."), *TestNameToPrint, *Expected.ToString(), *Actual.ToString()), 1);
			bSuccess = false;
		}
	}


	// Name collision test
	int32 Count = 0;
	TSet<FString> Keys;
	FRandomStream RandomStream;
	RandomStream.Initialize(42);
	for (uint8 TestEncoding = static_cast<uint8>(EEncoding::None); TestEncoding < static_cast<uint8>(EEncoding::Max); ++TestEncoding)
	{
		for (uint8 TestColorSpace = static_cast<uint8>(EColorSpace::None); TestColorSpace <= static_cast<uint8>(EColorSpace::Max); ++TestColorSpace)
		{
			for (uint8 TestChromatic = 0; TestChromatic < 2; ++TestChromatic)
			{
				FOpenColorIOWrapperSourceColorSettings TestSettings;
				TestSettings.EncodingOverride = static_cast<EEncoding>(TestEncoding);

				if (TestColorSpace < static_cast<uint8>(EColorSpace::Max))
				{
					TestSettings.ColorSpace = static_cast<EColorSpace>(TestColorSpace);
				}
				else
				{
					// We locally use EColorSpace::Max as a test custom color space.
					TStaticArray<FVector2d, 4> RandomChromaticities;
					RandomChromaticities[0] = FVector2d(RandomStream.GetUnitVector());
					RandomChromaticities[1] = FVector2d(RandomStream.GetUnitVector());
					RandomChromaticities[2] = FVector2d(RandomStream.GetUnitVector());
					RandomChromaticities[3] = FVector2d(RandomStream.GetUnitVector());

					TestSettings.ColorSpaceOverride = MoveTemp(RandomChromaticities);
				}
				
				TestSettings.ChromaticAdaptationMethod = static_cast<EChromaticAdaptationMethod>(TestChromatic);

				Keys.Add(FOpenColorIOWrapperProcessor::GetTransformToWorkingColorSpaceName(TestSettings));
				Count++;
			}
		}
	}

	bSuccess &= TestEqual(TEXT("OpenColorIO: Name hash collision test"), Keys.Num(), Count);

	return bSuccess;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FOpenColorIOHLGTest, "System.OpenColorIO.HLGDecodeMatchesEngine", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FOpenColorIOHLGTest::RunTest(const FString& Parameters)
{
	using namespace UE::Color;

	// Test multiple values across the HLG range, including the piecewise boundary at 0.5
	// and values in both the quadratic (<=0.5) and exponential (>0.5) regions.
	const float TestValues[] = { 0.0f, 0.1f, 0.25f, 0.5f, 0.75f, 0.9f, 1.0f };

	FOpenColorIOWrapperSourceColorSettings HLGSettings;
	HLGSettings.EncodingOverride = EEncoding::HLG;

	FOpenColorIOWrapperProcessor Processor = FOpenColorIOWrapperProcessor::CreateTransformToWorkingColorSpace(HLGSettings);

	bool bSuccess = true;

	for (float V : TestValues)
	{
		FLinearColor TestColor(V, V, V, 1.0f);
		FLinearColor Expected = UE::Color::Decode(EEncoding::HLG, TestColor);

		FLinearColor Actual = TestColor;
		Processor.TransformColor(Actual);

		const float Tolerance = FMath::Max(UE_KINDA_SMALL_NUMBER, UE_KINDA_SMALL_NUMBER * 0.5f * (Actual.R + Expected.R));

		if (!Actual.Equals(Expected, Tolerance))
		{
			AddError(FString::Printf(TEXT("HLG decode mismatch at value %.4f: expected %s, got %s."), V, *Expected.ToString(), *Actual.ToString()), 1);
			bSuccess = false;
		}
	}

	return bSuccess;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FOpenColorIOST2084Test, "System.OpenColorIO.ST2084DecodeMatchesEngine", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FOpenColorIOST2084Test::RunTest(const FString& Parameters)
{
	using namespace UE::Color;

	// ST2084 samples should match Decode(EEncoding::ST2084, ...) which returns absolute nits.
	// Test values cover the non-linear PQ range including black, mid-tones, and peak.
	const float TestValues[] = { 0.0f, 0.1f, 0.25f, 0.5f, 0.75f, 0.9f, 1.0f };

	FOpenColorIOWrapperSourceColorSettings ST2084Settings;
	ST2084Settings.EncodingOverride = EEncoding::ST2084;

	FOpenColorIOWrapperProcessor Processor = FOpenColorIOWrapperProcessor::CreateTransformToWorkingColorSpace(ST2084Settings);

	bool bSuccess = true;

	for (float V : TestValues)
	{
		FLinearColor TestColor(V, V, V, 1.0f);
		FLinearColor Expected = UE::Color::Decode(EEncoding::ST2084, TestColor);

		FLinearColor Actual = TestColor;
		Processor.TransformColor(Actual);

		const float Tolerance = FMath::Max(UE_KINDA_SMALL_NUMBER, UE_KINDA_SMALL_NUMBER * 0.5f * (Actual.R + Expected.R));

		if (!Actual.Equals(Expected, Tolerance))
		{
			AddError(FString::Printf(TEXT("ST2084 decode mismatch at value %.4f: expected %s, got %s."), V, *Expected.ToString(), *Actual.ToString()), 1);
			bSuccess = false;
		}
	}

	return bSuccess;
}

#endif