// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_OCIO

#include "CoreTypes.h"
#include "Containers/ContainersFwd.h"
#include "Containers/StaticArray.h"
#include "Math/Vector.h"
#include "Templates/PimplPtr.h"
#include "Templates/UniquePtr.h"

#include "ColorManagement/ColorManagementDefines.h"

struct FImageView;
enum TextureFilter : int;

namespace OpenColorIOWrapper
{
	/** Color space name of the engine's working color space we insert in OpenColorIO configs. */
	constexpr const TCHAR* GetWorkingColorSpaceName()
	{
		return TEXT("Working Color Space");
	}

	/** Default generated shader function name. */
	constexpr const TCHAR* GetShaderFunctionName()
	{
		return TEXT("OCIOConvert");
	}

	/** Default LUT size used in the legacy gpu processor */
	static constexpr uint32 Legacy3dEdgeLength = 65;

	/** Get the OpenColorIO version string. */
	OPENCOLORIOWRAPPER_API const TCHAR* GetVersion();

	/** Get the OpenColorIO version hex value. */
	OPENCOLORIOWRAPPER_API uint32 GetVersionHex();

	/** Calls the native function of the same name. */
	OPENCOLORIOWRAPPER_API void ClearAllCaches();
}


/**
 * Wrapper around library-native config object.
 */
class FOpenColorIOWrapperConfig
{
public:

	/** Config initialization options. */
	enum EAddWorkingColorSpaceOption
	{
		WCS_None,
		// Add the working color space as ACES 2065-1 so that we can target it with an extra conversion from our config if it defines aces_interchange.
		WCS_AsInterchangeSpace
	};

	/** Constructor. */
	FOpenColorIOWrapperConfig();

	/**
	* Constructor.
	* 
	* @param InFilePath Config absolute file path.
	* @param InOptions Initialization options.
	*/
	OPENCOLORIOWRAPPER_API explicit FOpenColorIOWrapperConfig(FStringView InFilePath, EAddWorkingColorSpaceOption InOption = WCS_None);

	virtual ~FOpenColorIOWrapperConfig() = default;

	/** Valid when the native config has been successfully created and isn't null. */
	OPENCOLORIOWRAPPER_API bool IsValid() const;
	
	/** Get the number of color spaces in the configuration. */
	OPENCOLORIOWRAPPER_API int32 GetNumColorSpaces() const;
	
	/** Get a color space name at an index. */
	OPENCOLORIOWRAPPER_API FString GetColorSpaceName(int32 Index) const;
	
	/** Get the index of a color space, -1 if none. */
	OPENCOLORIOWRAPPER_API int32 GetColorSpaceIndex(const TCHAR* InColorSpaceName);
	
	/** Get the family name for a color space. */
	OPENCOLORIOWRAPPER_API FString GetColorSpaceFamilyName(const TCHAR* InColorSpaceName) const;

	/** Get a color space description. */
	OPENCOLORIOWRAPPER_API FString GetColorSpaceDescription(const TCHAR* InColorSpaceName) const;
	
	/** Get the number of displays in the configuration. */
	OPENCOLORIOWRAPPER_API int32 GetNumDisplays() const;
	
	/** Get a display name at an index. */
	OPENCOLORIOWRAPPER_API FString GetDisplayName(int32 Index) const;
	
	/** Get the number of views for a display. */
	OPENCOLORIOWRAPPER_API int32 GetNumViews(const TCHAR* InDisplayName) const;
	
	/** Get a view name for its display and index. */
	OPENCOLORIOWRAPPER_API FString GetViewName(const TCHAR* InDisplayName, int32 Index) const;

	/** Get a display-view transform name. */
	OPENCOLORIOWRAPPER_API FString GetDisplayViewTransformName(const TCHAR* InDisplayName, const TCHAR* InViewName) const;

	/** Get a display-view transform name. */
	OPENCOLORIOWRAPPER_API FString GetDisplayViewDescription(const TCHAR* InDisplayName, const TCHAR* InViewName) const;

	/** Get a display-view transform description. */
	OPENCOLORIOWRAPPER_API FString GetDisplayViewTransformDescription(const TCHAR* InDisplayName, const TCHAR* InViewName) const;

	/** Returns the current context key-value strings. */
	OPENCOLORIOWRAPPER_API TMap<FString, FString> GetCurrentContextStringVars() const;

	/** Get the string hash of the config. */
	OPENCOLORIOWRAPPER_API FString GetCacheID() const;

	/** Returns a config debug string. */
	OPENCOLORIOWRAPPER_API FString GetDebugString() const;

	/** Returns all the available built-in config name/UI name pairs. */
	OPENCOLORIOWRAPPER_API TMap<FString, FString> GetBuiltInConfigNames(bool bRecommendedOnly, bool bShortenUIName) const;

	/** Make a config path forward-compatible by replacing special built-in config names with the current name. */
	OPENCOLORIOWRAPPER_API static FString ResolveConfigPath(const TCHAR* InOriginalPath) noexcept;

protected:

	TPimplPtr<struct FOpenColorIOConfigPimpl, EPimplPtrMode::DeepCopy> Pimpl;

	friend class FOpenColorIOWrapperProcessor;
	friend class FOpenColorIOWrapperGPUProcessor;
};

/* Matches FTextureSourceColorSettings in Engine/Texture.h */
struct FOpenColorIOWrapperSourceColorSettings
{
	/** Source encoding of the texture, exposing more options than just sRGB. */
	UE::Color::EEncoding EncodingOverride = UE::Color::EEncoding::None;

	/** Source color space of the texture. Leave to None for custom color space. */
	UE::Color::EColorSpace ColorSpace = UE::Color::EColorSpace::None;

	/** Optional chromaticity coordinates of a custom source color space. */
	TOptional<TStaticArray<FVector2d, 4>> ColorSpaceOverride = {};

	/** Chromatic adaption method applied if the source white point differs from the working color space white point. */
	UE::Color::EChromaticAdaptationMethod ChromaticAdaptationMethod = UE::Color::EChromaticAdaptationMethod::None;
};

/**
 * Wrapper around library-native config object for module-specific use cases.
 * This uses the built-in studio config as a base.
 */
class FOpenColorIOWrapperEngineConfig : public FOpenColorIOWrapperConfig
{
public:
	/**
	* Constructor.
	*/
	FOpenColorIOWrapperEngineConfig();

	FOpenColorIOWrapperEngineConfig(FStringView InFilePath, EAddWorkingColorSpaceOption InOption = WCS_None) = delete;
};

/**
 * Wrapper around library-native processor object.
 * This class can also be used for CPU transforms. It will allocate CPU processors as needed, and reuse them since they are by default cached by the library.
 */
class FOpenColorIOWrapperProcessor final
{
public:

	/**
	* Empty constructor.
	*/
	OPENCOLORIOWRAPPER_API FOpenColorIOWrapperProcessor();

	/**
	* Constructor.
	*
	* @param InConfig Owner config.
	* @param InSourceColorSpace Source color space name.
	* @param InDestinationColorSpace Destination color space name.
	* @param InContextKeyValues (Optional) Additional context modifiers.
	*/
	OPENCOLORIOWRAPPER_API explicit FOpenColorIOWrapperProcessor(
		const FOpenColorIOWrapperConfig* InConfig,
		FStringView InSourceColorSpace,
		FStringView InDestinationColorSpace,
		const TMap<FString, FString>& InContextKeyValues = {}
	);

	/**
	* Constructor.
	*
	* @param InConfig Owner config.
	* @param InSourceColorSpace Source color space name.
	* @param InDisplay Display name in display-view transform.
	* @param InView View name in display-view transform.
	* @param bInverseDirection Flag for inverse transform direction.
	* @param InContextKeyValues (Optional) Additional context modifiers.
	*/
	OPENCOLORIOWRAPPER_API explicit FOpenColorIOWrapperProcessor(
		const FOpenColorIOWrapperConfig* InConfig,
		FStringView InSourceColorSpace,
		FStringView InDisplay,
		FStringView InView,
		bool bInverseDirection = false,
		const TMap<FString, FString>& InContextKeyValues = {}
	);

	/**
	* Constructor.
	*
	* @param InConfig Owner config.
	* @param InNamedTransform Transform name.
	* @param bInverseDirection Flag for inverse transform direction.
	* @param InContextKeyValues (Optional) Additional context modifiers.
	*/
	OPENCOLORIOWRAPPER_API explicit FOpenColorIOWrapperProcessor(
		const FOpenColorIOWrapperConfig* InConfig,
		FStringView InNamedTransform,
		bool bInverseDirection = false,
		const TMap<FString, FString>& InContextKeyValues = {}
	);

	/**
	* Create a processor from source color settings to working color space.
	* @param InColorSettings Source encoding and colorspace.
	* @return Processor of the transform
	*/
	OPENCOLORIOWRAPPER_API static FOpenColorIOWrapperProcessor CreateTransformToWorkingColorSpace(
		const FOpenColorIOWrapperSourceColorSettings& InSourceColorSettings
	);

	/** Valid when the processor has been successfully created and isn't null. */
	OPENCOLORIOWRAPPER_API bool IsValid() const;

	/** Get the string hash of the processor. */
	OPENCOLORIOWRAPPER_API FString GetCacheID() const;

	/**
	* Get the generated transform name from source color settings to working color space.
	* @param InColorSettings Source encoding and colorspace.
	* @return Generated name of the transform
	*/
	OPENCOLORIOWRAPPER_API static FString GetTransformToWorkingColorSpaceName(const FOpenColorIOWrapperSourceColorSettings& InSourceColorSettings);

	/** Apply the CPU color transform in-place to the specified color. */
	OPENCOLORIOWRAPPER_API bool TransformColor(FLinearColor& InOutColor) const;

	/** Apply the CPU color transform in-place to the specified image. */
	OPENCOLORIOWRAPPER_API bool TransformImage(const FImageView& InOutImage) const;

	/** Apply the CPU color transform from the source image to the destination image.
	 * (The destination FImageView is const but what it points at is not.)
	 * 
	 * Note: This function is currently less optimal, as it is not parallelized.
	 */
	OPENCOLORIOWRAPPER_API bool TransformImage(const FImageView& SrcImage, const FImageView& DestImage) const;

private:

	TPimplPtr<struct FOpenColorIOProcessorPimpl, EPimplPtrMode::DeepCopy> Pimpl;

	friend class FOpenColorIOWrapperGPUProcessor;
};

/**
 * Wrapper around library-native GPU processor object.
 */
class FOpenColorIOWrapperGPUProcessor final
{
public:

	/**
	* Constructor.
	*
	* @param InProcessor Parent processor.
	* @param InOptions Initialization options.
	*/
	OPENCOLORIOWRAPPER_API explicit FOpenColorIOWrapperGPUProcessor(
		FOpenColorIOWrapperProcessor InProcessor,
		bool bUseLegacy = false
	);

	/** Valid when the processor has been successfully created and isn't null. */
	OPENCOLORIOWRAPPER_API bool IsValid() const;

	/**
	* Get the generated shader.
	*
	* @param OutShaderCacheID Generated shader hash string.
	* @param OutShaderCode Generated shader text.
	* @return True when the result is valid.
	*/
	OPENCOLORIOWRAPPER_API bool GetShader(FString& OutShaderCacheID, FString& OutShaderCode) const;

	/** Get the number of 3D LUT textures. */
	OPENCOLORIOWRAPPER_API uint32 GetNum3DTextures() const;

	/**
	* Get the index-specific 3D LUT texture used by the shader transform.
	*
	* @param InIndex Index of the texture.
	* @param OutName Shader parameter name of the texture.
	* @param OutEdgeLength Size of the texture.
	* @param OutTextureFilter Texture filter type.
	* @param OutData Raw texel data.
	* @return True when the result is valid.
	*/
	OPENCOLORIOWRAPPER_API bool Get3DTexture(uint32 InIndex, FName& OutName, uint32& OutEdgeLength, TextureFilter& OutTextureFilter, const float*& OutData) const;

	/** Get the number of 2D LUT textures. (1D resources are disabled in our case.) */
	OPENCOLORIOWRAPPER_API uint32 GetNumTextures() const;

	/**
	* Get the index-specific 2D LUT texture used by the shader transform.
	*
	* @param InIndex Index of the texture.
	* @param OutName Shader parameter name of the texture.
	* @param OutWidth Width of the texture.
	* @param OutHeight Height of the texture.
	* @param OutTextureFilter Texture filter type.
	* @param bOutRedChannelOnly Flag indicating whether the texture has a single channel or is RGB.
	* @param OutData Raw texel data.
	* @return True when the result is valid.
	*/
	OPENCOLORIOWRAPPER_API bool GetTexture(uint32 InIndex, FName& OutName, uint32& OutWidth, uint32& OutHeight, TextureFilter& OutTextureFilter, bool& bOutRedChannelOnly, const float*& OutData) const;

	/** Get the string hash of the gpu processor. */
	OPENCOLORIOWRAPPER_API FString GetCacheID() const;

private:

	const FOpenColorIOWrapperProcessor ParentProcessor;

	TPimplPtr<struct FOpenColorIOGPUProcessorPimpl, EPimplPtrMode::DeepCopy> GPUPimpl;
};

#endif // WITH_OCIO
