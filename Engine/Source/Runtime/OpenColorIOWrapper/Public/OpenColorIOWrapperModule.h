// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleManager.h"
#include "Logging/LogMacros.h"

class FOpenColorIOWrapperEngineConfig;

DECLARE_LOG_CATEGORY_EXTERN(LogOpenColorIOWrapper, Log, All);

/**
 * Interface for the OpenColorIO module.
 */
class IOpenColorIOWrapperModule : public IModuleInterface
{
public:
	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static IOpenColorIOWrapperModule& Get()
	{
		static const FName ModuleName = "OpenColorIOWrapper";
		return FModuleManager::LoadModuleChecked<IOpenColorIOWrapperModule>(ModuleName);
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() during shutdown if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static bool IsAvailable()
	{
		static const FName ModuleName = "OpenColorIOWrapper";
		return FModuleManager::Get().IsModuleLoaded(ModuleName);
	}

#if WITH_OCIO
	/**
	 * Returns a minimal dynamically-created native config for engine (working color spaces) conversions.
	 */
	virtual FOpenColorIOWrapperEngineConfig& GetEngineBuiltInConfig() = 0;
#endif //WITH_OCIO

	/** Virtual destructor */
	virtual ~IOpenColorIOWrapperModule() = default;
};
